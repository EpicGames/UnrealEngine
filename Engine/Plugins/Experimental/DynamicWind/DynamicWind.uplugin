{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Dynamic Wind",
	"Description": "Extremely experimental dynamic wind support for Nanite foliage.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DynamicWind",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "DynamicWindEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}