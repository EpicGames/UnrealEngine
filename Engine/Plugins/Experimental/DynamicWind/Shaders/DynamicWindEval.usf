// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/BoneTransform.ush"
#include "/Engine/Shared/SkinningDefinitions.h"
#include "/Engine/Private/ComputeShaderUtils.ush"
#include "/Engine/Private/Skinning/SkinningDataDecode.ush"

#include "Shared/DynamicWindCommon.ush"

// Enable to debug always outputting identity bones
#define FORCE_IDENTITY_OUTPUT 0

StructuredBuffer<FDynamicWindSkeletonData> SkeletonBuffer;
StructuredBuffer<FDynamicWindBlockHeader> HeaderBuffer;
ByteAddressBuffer BoneDataBuffer;
ByteAddressBuffer BoneMapBuffer;
FCompressedBoneTransformRWBuffer TransformBuffer;
Texture2D<float4> WindTexture;
SamplerState WindSampler;

float Time;
float DeltaTime;
float WindSpeed;
float WindAmplitude;
float3 WindDirection;
float4 Debug;

// === WIND PARAMETERS ===============================================================================================================================

// Wind speed and amplitude modulations
#define DYNAMIC_WIND_SPEED_MOD 1.25f
#define DYNAMIC_WIND_AMPLITUDE_MOD 3.0f

// Flat value added to the final amplitude
#define DYNAMIC_WIND_BEND_OFFSET 0.1f

// Attenuation modifier based on the wind direction
#define DYNAMIC_WIND_ALIGN_ATTENUATION 0.5f

// SPECIFIC FOR BRANCHES

// Attenuation modifier based on the bone height (0-1 normalized on skeleton height)
#define DYNAMIC_WIND_HEIGHT_ATTENUATION 0.1f

// Modulation of the angle of the torsion rotation of a branch
#define DYNAMIC_WIND_TORSION_MOD 0.5f

// Attenuation by branch length
#define DYNAMIC_WIND_LONG_BRANCH_SPEED_MOD 0.5f
#define DYNAMIC_WIND_LONG_BRANCH_MAX 500.0f
#define DYNAMIC_WIND_LONG_BRANCH_MIN 100.0f

// Gust speed
#define DYNAMIC_WIND_GUST_SPEED_MIN 0.05f
#define DYNAMIC_WIND_GUST_SPEED_MAX 0.1f

uint GetBoneDataAddress(uint BoneDataOffset, uint BoneIndex)
{
	return (BoneDataOffset + BoneIndex) * uint(sizeof(FDynamicWindBoneData));
}

float RemapValue(float Value, float InLow, float InHigh, float OutLow, float OutHigh)
{
	return OutLow + (Value - InLow) * (OutHigh - OutLow) / (InHigh - InLow);
}

FQuat QuatAttenuate(FQuat Q1, half Attenuation)
{
	const FQuat Q0 = QuatIdentity();
	const half C = dot(Q0, Q1);
	return normalize(lerp(Q0, C < 0.0f ? -Q1 : Q1, Attenuation));
}

float WindFlow(const float RandomValue, const float Distance = 0.0f, const float BlendByGroup = 0.0f, const float ChainLength = 0.0f, float TrunkBlend = 0.0f)
{	
	// Reducing the speed on lower branches to simulate wood thickness and weight
	const float RemappedBranchLength = RemapValue(ChainLength, DYNAMIC_WIND_LONG_BRANCH_MAX, DYNAMIC_WIND_LONG_BRANCH_MIN, DYNAMIC_WIND_LONG_BRANCH_SPEED_MOD, 1.0f);
	const float SpeedByLengthModulation = max(TrunkBlend, clamp(RemappedBranchLength, DYNAMIC_WIND_LONG_BRANCH_SPEED_MOD, 1.0f));
	
	// 100.0f is a manual modulation to keep the speed inside human-readable range
	const float BiasedWindSpeed = WindSpeed / 100.0f;
	const float FlowSpeed = Time * BiasedWindSpeed * SpeedByLengthModulation * DYNAMIC_WIND_SPEED_MOD;
	const float2 UV = float2(RandomValue, frac(Distance * RandomValue)) - float2(frac(FlowSpeed), 0.0f);
	
	const float4 Sample = WindTexture.SampleLevel(WindSampler, UV, 0);

	// Fractal Brownian noise approximation
	float FinalIntensity = Sample.x + lerp(0.0f, Sample.w * 0.5f, BlendByGroup);
	FinalIntensity = (DYNAMIC_WIND_BEND_OFFSET + FinalIntensity) * DYNAMIC_WIND_AMPLITUDE_MOD * WindSpeed / 100.0f;

	return FinalIntensity;
}

float WindWave(float T)
{
	return cos(PI * T) * cos(3.0f * PI * T) * cos(5.0f * PI * T) * cos(7.0f * PI * T);
}

float RampStrength(float X)
{
	return X / (X * X + 1.0f);
}

[numthreads(BONES_PER_GROUP, 1, 1)]
void WindEvalCS(uint3 GroupId : SV_GroupID, uint GroupThreadIndex : SV_GroupIndex)
{
	const uint GlobalBlockIndex	= GroupId.x;
	const uint BoneIndexInBlock	= GroupThreadIndex;

	const FDynamicWindBlockHeader Header = HeaderBuffer[GlobalBlockIndex];
	const uint BlockNumBones = GetBlockNumBones(Header);
	const uint BlockBoneOffset = GetBlockBoneOffset(Header);
	const uint BlockDirectionalityIndex = GetBlockDirectionalityIndex(Header);
	const uint TotalTransformCount = GetTotalTransformCount(Header);

	if (BoneIndexInBlock >= BlockNumBones)
	{
		return;
	}

	if (Header.SkeletonDataIndex == ~uint(0) || FORCE_IDENTITY_OUTPUT)
	{
		const float4x3 Identity =
		{
			float3(1, 0, 0),
			float3(0, 1, 0),
			float3(0, 0, 1),
			float3(0, 0, 0)
		};
		StoreCompressedBoneTransform(TransformBuffer, Header.BlockDstTransformOffset, BoneIndexInBlock, Identity);
		StoreCompressedBoneTransform(TransformBuffer, Header.BlockDstTransformOffset, BoneIndexInBlock + TotalTransformCount, Identity);

		return;
	}

	FDynamicWindSkeletonData SkeletonData = SkeletonBuffer[Header.SkeletonDataIndex];
	const uint MaxSimulationGroupIndex = GetMaxSimulationGroupIndex(SkeletonData);
	const uint IsGroundCover = GetIsGroundCover(SkeletonData);

	const uint TransformSlotIndex = BlockBoneOffset + BoneIndexInBlock;
	const uint SkeletonBoneIndex = (Header.BoneMapBufferOffset != ~uint(0))
		? BoneMapBuffer.Load(Header.BoneMapBufferOffset + TransformSlotIndex * 4)
		: TransformSlotIndex;
	const uint BoneDataAddress = GetBoneDataAddress(SkeletonData.BoneDataOffset, SkeletonBoneIndex);
	const FDynamicWindBoneData BoneData = BoneDataBuffer.Load<FDynamicWindBoneData>(BoneDataAddress);

#if USE_SIN_WIND
	const float3 WindRotationVector = cross(float3(0.0f, 0.0f, 1.0f), WindDirection);
	FBoneTransform BoneTransform = MakeBoneTransformIdentity();	
	BoneTransform.Rotation = QuatNormalize(QuatFromAxisAngle(WindRotationVector, sin(Time * 0.25f) * 0.005f * WindSpeed));
#else		
	FBoneTransform BoneTransform = MakeBoneTransformIdentity();
	float3 PositionAccumulation = float3(0.0f, 0.0f, 0.0f);
	float3 LastBoneBindPosition = BoneData.BindPosePosition;

	float SliceAngle = 2.0f * PI / float(DYNAMIC_WIND_DIRECTIONALITY_SLICES);

	for (uint CurrentBoneDataAddress = BoneDataAddress, ParentBoneIndex = SkeletonBoneIndex; ParentBoneIndex != DYNAMIC_WIND_INVALID_BONE_INDEX;)
	{
		const FDynamicWindBoneData CurrentBoneData = BoneDataBuffer.Load<FDynamicWindBoneData>(CurrentBoneDataAddress);

		// === ROTATION AXIS CALCULATIONS ========================================================================================================
		const float3 SectionWindDirection = QuatRotateVector(QuatConjugate(QuatFromAxisAngle(float3(0.0f, 0.0f, 1.0f), BlockDirectionalityIndex * SliceAngle)), WindDirection);
		const float3 AdjustedBoneForward = QuatRotateVector(CurrentBoneData.BindPoseRotation, float3(1.0f, 0.0f, 0.0f));
			
		// Prevent error when the bone forward and wind direction are collinear
		const float3 WindHorRotVector = abs(dot(AdjustedBoneForward, SectionWindDirection)) > 0.999f ?
			normalize(cross(AdjustedBoneForward, float3(0.0f, 0.0f, 1.0f))) :
			normalize(cross(AdjustedBoneForward, SectionWindDirection));
			
		const float3 WindVerRotVector = normalize(cross(AdjustedBoneForward, float3(0.0f, 0.0f, 1.0f)));
			
		const float WindDirectionAlignment = 1.0f - saturate(dot(AdjustedBoneForward, SectionWindDirection)) * DYNAMIC_WIND_ALIGN_ATTENUATION;
		const float WindDirectionOrthogonality = 1.0f - abs(dot(AdjustedBoneForward, SectionWindDirection));
			
		const float HeightGrad = saturate(CurrentBoneData.BindPosePosition.z / SkeletonData.SkeletonHeight);
		// RandBBSfloat(0) returns 0, which result in an uniform rotation for all branches
		// SkeletonDataIndex is 0 for the first registered skeleton, hence the +1
		const float RandomValue = RandBBSfloat(((Header.SkeletonDataIndex << 8) | BlockDirectionalityIndex) + 1);
			
		float GustSpeed = lerp(DYNAMIC_WIND_GUST_SPEED_MIN, DYNAMIC_WIND_GUST_SPEED_MAX, WindSpeed / 100.0f);
		float GustStrength = RampStrength(WindSpeed / 100.0f);

		FQuat PartialRotation;

		if (CurrentBoneData.bIsTrunkBone)
		{
			float WindX = WindWave(GustSpeed * Time + RandomValue * 17) + 0.75f;
			WindX *= lerp(0.3f, 0.6f, HeightGrad);
			WindX *= 0.06f * GustStrength;
			WindX += 0.02f * Pow4(WindSpeed / 100.0f);
				
			float WindY = WindWave(GustSpeed * Time + RandomValue * 17 + 23);
			WindY *= lerp(0.5f, 0.4f, HeightGrad);
			WindY *= 0.03f * GustStrength;
			WindY *= 0.25f;
				
			float3 BendX = cross(float3(0.0f, 0.0f, 1.0f), SectionWindDirection);
			float3 BendY = cross(float3(0.0f, 0.0f, 1.0f), BendX);
				
			FQuat RotateX = QuatFromAxisAngle(BendX, WindX);
			FQuat RotateY = QuatFromAxisAngle(BendY, WindY);
				
			PartialRotation = QuatMultiply(RotateX, RotateY);
			PartialRotation = QuatAttenuate(PartialRotation, (1.0f - SkeletonData.GustAttenuation) * WindAmplitude);
		}
		else // Branches
		{
			const float DistanceFromOrigin = distance(CurrentBoneData.BindPosePosition, float3(0.0f, 0.0f, 0.0f));
			const float BlendByGroup = max(CurrentBoneData.SimulationGroupIndex, 0) / (MaxSimulationGroupIndex + 1);
			const float TrunkBlend = (float)CurrentBoneData.bIsTrunkBone;
				
			float Gust = GustStrength * max(0, WindWave(GustSpeed * (Time + 0.5f) + RandomValue * 17));

			float WindFlowIntensity = WindFlow(RandomValue, DistanceFromOrigin, BlendByGroup, CurrentBoneData.BoneChainLength, TrunkBlend);
			WindFlowIntensity *= lerp(0.2f, 0.4f, Pow2(HeightGrad));
			WindFlowIntensity *= Gust * (4 + 8 * cos(0.4f * Time + RandomValue * 17 + frac(DistanceFromOrigin * RandomValue) * 1777)) + 1.0f;

			// Soft clip
			WindFlowIntensity /= PI;
			WindFlowIntensity /= WindFlowIntensity + 1;
			WindFlowIntensity *= PI;

			PartialRotation = QuatFromAxisAngle(WindHorRotVector, WindFlowIntensity * CurrentBoneData.Influence * WindDirectionAlignment);
			
			// Add torsion to branches positioned orthogonally to wind direction
			const FQuat TorsionRotation = QuatFromAxisAngle(AdjustedBoneForward, DYNAMIC_WIND_TORSION_MOD * WindFlowIntensity * CurrentBoneData.Influence * WindDirectionOrthogonality);
			PartialRotation = QuatMultiply(PartialRotation, TorsionRotation);
				
			float WindBounce = WindWave(0.2f * Time + RandomValue * 13 + frac(DistanceFromOrigin * RandomValue) * 1777);
			WindBounce *= 0.25f * RampStrength(WindSpeed / 100.0f);
			WindBounce *= Gust * 8 + 1;
				
			FQuat VerticalRotation = QuatFromAxisAngle(WindVerRotVector, WindBounce * CurrentBoneData.Influence);
			PartialRotation = QuatMultiply(VerticalRotation, PartialRotation);

			// Attenuate branch final rotation on lower branches
			// It's here and not on the branch section because I want to influence all the rotation on the branch
			const float RawHeightBlend = CurrentBoneData.BindPosePosition.z / SkeletonData.SkeletonHeight;
			const float FinalHeightBlend = max(IsGroundCover, RemapValue(RawHeightBlend, 0.0f, 1.0f, DYNAMIC_WIND_HEIGHT_ATTENUATION, 1.0f));
			PartialRotation = QuatAttenuate(PartialRotation, FinalHeightBlend * WindAmplitude);
		}
			
		const FQuat CurrentRotation = QuatMultiply(BoneTransform.Rotation, PartialRotation);
			
		PositionAccumulation = QuatRotateVector(PartialRotation, PositionAccumulation + LastBoneBindPosition - CurrentBoneData.BindPosePosition);
		BoneTransform.Rotation = CurrentRotation;
		LastBoneBindPosition = CurrentBoneData.BindPosePosition;
			
		ParentBoneIndex = CurrentBoneData.ParentBoneIndex;
		CurrentBoneDataAddress = GetBoneDataAddress(SkeletonData.BoneDataOffset, CurrentBoneData.ParentBoneIndex);
	}
		
	BoneTransform.Location = PositionAccumulation - QuatRotateVector(BoneTransform.Rotation, BoneData.BindPosePosition);
#endif

	const float4x3 CurrentTransform = (float4x3)MatrixFromBoneTransform(BoneTransform);
	const float4x3 PreviousTransform = LoadCompressedBoneTransform(TransformBuffer, Header.BlockDstTransformOffset, BoneIndexInBlock);
	
	// TODO: Handle first frame by writing CurrentTransform for both

	StoreCompressedBoneTransform(TransformBuffer, Header.BlockDstTransformOffset, BoneIndexInBlock, CurrentTransform);
	StoreCompressedBoneTransform(TransformBuffer, Header.BlockDstTransformOffset, BoneIndexInBlock + TotalTransformCount, PreviousTransform);
}