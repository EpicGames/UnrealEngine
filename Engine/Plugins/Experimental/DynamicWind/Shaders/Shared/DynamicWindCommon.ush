// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define DYNAMIC_WIND_INVALID_BONE_INDEX 65535

#define DYNAMIC_WIND_DIRECTIONALITY_SLICES 8

#ifdef __cplusplus
namespace UE::HLSL
{
#endif //__cplusplus

struct FDynamicWindSkeletonData
{
	/** The index of the first element of the FDynamicWindBoneData for this skeleton */
	uint BoneDataOffset;

	/** The height (in units) of the skeleton */
	float SkeletonHeight;

	/** How much to attenuate trunk bones in the skeleton (0-1) */
	float GustAttenuation;

	/** Packed: MaxSimulationGroupIndex (bits 0-30), bIsGroundCover (bit 31) */
	uint PackedSimGroupAndGroundCover;
};

struct FDynamicWindBoneData
{
	/** The index of this bone's parent bone, in the skeleton */
	uint ParentBoneIndex;

	/** The wind influence of this bone */
	float Influence;

	/** The simulation group for this bone */
	int SimulationGroupIndex;

	/** The length of bone chain that this bone is a member of */
	float BoneChainLength;

	/** Initial local-space orientation of the bone */
	float4 BindPoseRotation;

	/** Initial local-space position of the bone */
	float3 BindPosePosition;

	/** Whether this bone is to be considered part of a tree trunk */
	uint bIsTrunkBone;
};

struct FDynamicWindBlockHeader
{	
	/** Index of the related FDynamicWindSkeletonData for the block */
	uint SkeletonDataIndex;

	/** Byte offset of the transforms to be written in the transform buffer */
	uint BlockDstTransformOffset;

	/** Packed: BlockNumBones (bits 0-7), BlockBoneOffset (bits 8-31) */
	uint PackedNumBonesAndOffset;

	/** Packed: BlockDirectionalityIndex (bits 0-7), TotalTransformCount (bits 8-31) */
	uint PackedDirectionalityAndTransformCount;

	/** Byte offset of the bone map in the bone map buffer, or ~0u if not present */
	uint BoneMapBufferOffset;
};

#ifdef __cplusplus
} // namespace UE::HLSL

using FDynamicWindSkeletonData = UE::HLSL::FDynamicWindSkeletonData;
using FDynamicWindBoneData = UE::HLSL::FDynamicWindBoneData;
using FDynamicWindBlockHeader = UE::HLSL::FDynamicWindBlockHeader;

// C++ setters for packed fields

inline void SetMaxSimulationGroupIndex(FDynamicWindSkeletonData& S, uint32 V) { S.PackedSimGroupAndGroundCover = (S.PackedSimGroupAndGroundCover & 0x80000000u) | (V & 0x7FFFFFFFu); }
inline void SetIsGroundCover(FDynamicWindSkeletonData& S, uint32 V) { S.PackedSimGroupAndGroundCover = (S.PackedSimGroupAndGroundCover & 0x7FFFFFFFu) | (V << 31); }

inline void SetBlockNumBones(FDynamicWindBlockHeader& H, uint32 V) { H.PackedNumBonesAndOffset = (H.PackedNumBonesAndOffset & 0xFFFFFF00u) | (V & 0xFFu); }
inline void SetBlockBoneOffset(FDynamicWindBlockHeader& H, uint32 V) { H.PackedNumBonesAndOffset = (H.PackedNumBonesAndOffset & 0xFFu) | (V << 8); }

inline void SetBlockDirectionalityIndex(FDynamicWindBlockHeader& H, uint32 V) { H.PackedDirectionalityAndTransformCount = (H.PackedDirectionalityAndTransformCount & 0xFFFFFF00u) | (V & 0xFFu); }
inline void SetTotalTransformCount(FDynamicWindBlockHeader& H, uint32 V) { H.PackedDirectionalityAndTransformCount = (H.PackedDirectionalityAndTransformCount & 0xFFu) | (V << 8); }

#else // HLSL accessors for packed fields

uint GetMaxSimulationGroupIndex(FDynamicWindSkeletonData S) { return S.PackedSimGroupAndGroundCover & 0x7FFFFFFFu; }
uint GetIsGroundCover(FDynamicWindSkeletonData S) { return S.PackedSimGroupAndGroundCover >> 31; }

uint GetBlockNumBones(FDynamicWindBlockHeader H) { return H.PackedNumBonesAndOffset & 0xFFu; }
uint GetBlockBoneOffset(FDynamicWindBlockHeader H) { return H.PackedNumBonesAndOffset >> 8; }
uint GetBlockDirectionalityIndex(FDynamicWindBlockHeader H) { return H.PackedDirectionalityAndTransformCount & 0xFFu; }
uint GetTotalTransformCount(FDynamicWindBlockHeader H) { return H.PackedDirectionalityAndTransformCount >> 8; }

#endif //__cplusplus
