// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class DynamicWind : ModuleRules
	{
		public DynamicWind(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PrivateIncludePaths.AddRange(
				new string[] {
					Path.Combine(GetModuleDirectory("Renderer"), "Private"),
					Path.Combine(PluginDirectory, "Shaders")
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Projects",
					"CoreUObject",
					"Engine",
					"SlateCore",
					"Slate",
					"RenderCore",
					"Renderer",
					"RHICore",
					"RHI",
					"RHI",
					"SlateCore",
					"Foliage"
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"DerivedDataCache",
						"ImageCore",
						"MeshBuilder",
						"MeshBuilderCommon",
						"MeshDescription",
						"MeshUtilities",
						"MeshUtilitiesCommon",
						"NaniteUtilities",
						"RawMesh",
						"StaticMeshDescription",
					}
				);

				PrivateIncludePathModuleNames.Add("NaniteBuilder");
				DynamicallyLoadedModuleNames.Add("NaniteBuilder");
			}
		}
	}
}
