// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

DYNAMICWIND_API DECLARE_LOG_CATEGORY_EXTERN(LogDynamicWind, Log, All);