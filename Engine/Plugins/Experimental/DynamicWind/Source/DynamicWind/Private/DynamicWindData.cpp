// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicWindData.h"
#include "DynamicWindSkeletalData.h"
#include "DynamicWindProvider.h"
#include "Engine/SkeletalMesh.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "RenderTransform.h"
#include "Math/MathFwd.h"
#include "ScenePrivate.h"
#include "SkeletalRenderPublic.h"
#include "SkinningSceneExtensionProxy.h"
#include "SkinningDefinitions.h"
#include "Components/InstancedSkinnedMeshComponent.h"

class FSceneInterface;

bool UDynamicWindData::IsEnabled() const
{
	return true;
}

const FGuid& UDynamicWindData::GetTransformProviderID() const
{
	static FGuid DynamicWindProviderId(DYNAMIC_WIND_TRANSFORM_PROVIDER_GUID);
	return DynamicWindProviderId;
}

bool UDynamicWindData::UsesSkeletonBatching() const
{
	return true;
}

uint32 UDynamicWindData::GetUniqueAnimationCount() const
{
	return DYNAMIC_WIND_DIRECTIONALITY_SLICES;
}

bool UDynamicWindData::HasAnimationBounds() const
{
	return false;
}

bool UDynamicWindData::GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const
{
	return false;
}

uint32 UDynamicWindData::GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const
{
	FTransform WorldSpaceTransform = FTransform(InstanceData.Transform) * ComponentTransform;
	const FVector EulerRotation = WorldSpaceTransform.GetRotation().Euler();
	int32 DirectionalityIndex = 0;
	{
		const int32 ZRoundDown = FMath::RoundToInt32(float(EulerRotation.Z) / 45.0f) * 45u;
		DirectionalityIndex = ((ZRoundDown + 360u) % 360u) / 45u;
	}

	return DirectionalityIndex * 2u;
}

FTransformProviderRenderProxy* UDynamicWindData::CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy) const
{
	return new FDynamicWindDataProxy(this, SceneExtensionProxy);
}

FDynamicWindDataProxy::FDynamicWindDataProxy(const UDynamicWindData* BankData, FInstancedSkinningSceneExtensionProxy* InSceneExtensionProxy)
	: SceneExtensionProxy(InSceneExtensionProxy)
{
	const USkinnedAsset* SkinnedAsset = SceneExtensionProxy->GetSkinnedAsset();
	check(SkinnedAsset != nullptr);
	
	const USkeletalMesh* SkeletalMesh = CastChecked<USkeletalMesh>(SkinnedAsset);
	SkeletalData = const_cast<USkeletalMesh*>(SkeletalMesh)->GetAssetUserData<UDynamicWindSkeletalData>();

	// GW-TODO: Need a validation method on the base data type to check per-proxy if ref pose fallback is needed.
}

FDynamicWindDataProxy::~FDynamicWindDataProxy()
{
}

void FDynamicWindDataProxy::CreateRenderThreadResources(FSceneInterface& InScene, FRHICommandListBase& RHICmdList)
{
	TransformProvider = FDynamicWindTransformProvider::FindForScene(&InScene);
	if (TransformProvider)
	{
		TransformProvider->RegisterProxy(SceneExtensionProxy, SkeletalData);
	}
}

void FDynamicWindDataProxy::DestroyRenderThreadResources()
{
	if (TransformProvider)
	{
		TransformProvider->UnregisterProxy(SceneExtensionProxy);
		TransformProvider = nullptr;
	}
}