// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicWindLog.h"

DEFINE_LOG_CATEGORY(LogDynamicWind);
