// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include <Interfaces/IPluginManager.h>
#include "Misc/Paths.h"
#include "ShaderCore.h"

#define LOCTEXT_NAMESPACE "DynamicWind"

class FDynamicWindModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FDynamicWindModule::OnPostEngineInit);
		FCoreDelegates::OnEnginePreExit.AddRaw(this, &FDynamicWindModule::OnEnginePreExit);

		const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("DynamicWind"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(TEXT("/Plugin/DynamicWind"), PluginShaderDir);
	}

	virtual void ShutdownModule() override
	{
		FCoreDelegates::OnEnginePreExit.RemoveAll(this);
		FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
	}

private:
	void OnPostEngineInit()
	{
	}

	void OnEnginePreExit()
	{
	}
};

IMPLEMENT_MODULE(FDynamicWindModule, DynamicWind);

#undef LOCTEXT_NAMESPACE
