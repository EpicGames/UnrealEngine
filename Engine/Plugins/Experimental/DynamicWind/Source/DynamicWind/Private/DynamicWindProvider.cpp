// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicWindProvider.h"
#include "DynamicWindSkeletalData.h"
#include "DynamicWindLog.h"
#include "Animation/Skeleton.h"
#include "Engine/Texture.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "SkinningSceneExtensionProxy.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "ScenePrivate.h"
#include "SkeletalRenderPublic.h"
#include "SkinningDefinitions.h"
#include "TextureResource.h"

static FGuid GDynamicWindTransformProviderId(DYNAMIC_WIND_TRANSFORM_PROVIDER_GUID);

DECLARE_GPU_STAT(DynamicWind);

namespace
{

static const FGuid DynamicWindTransformProviderId(DYNAMIC_WIND_TRANSFORM_PROVIDER_GUID);

TAutoConsoleVariable<float> CVarDynamicWindOverrideSpeed(
	TEXT("DynamicWind.OverrideSpeed"),
	-1.0f,
	TEXT("Overrides the wind speed. If this value is below 0.0f, then use the default wind speed set on the dynamic wind actor."),
	ECVF_RenderThreadSafe | ECVF_Default
);

TAutoConsoleVariable<float> CVarDynamicWindOverrideAmplitude(
	TEXT("DynamicWind.OverrideAmplitude"),
	-1.0f,
	TEXT("Overrides the wind amplitude. If this value is below 0.0f, then use the default wind amplitude set on the dynamic wind actor."),
	ECVF_RenderThreadSafe | ECVF_Default
);

TAutoConsoleVariable<float> CVarDynamicWindOverrideRateOfChange(
	TEXT("DynamicWind.OverrideRateOfChange"),
	0.5f,
	TEXT("Overrides the rate of change per second of the wind amplitude. It is used to smooth the wind amplitude over time to prevent animation popping."),
	ECVF_RenderThreadSafe | ECVF_Default
);

TAutoConsoleVariable<bool> CVarDynamicWindUseSine(
	TEXT("DynamicWind.UseSine"),
	false,
	TEXT("Forces dynamic wind to use a basic sine function to transform the bones."),
	ECVF_RenderThreadSafe | ECVF_Default
);

TAutoConsoleVariable<bool> CVarDynamicWindPreferAsyncCompute(
	TEXT("DynamicWind.PreferAsyncCompute"),
	false,
	TEXT("Run dynamic wind on async compute, if supported."),
	ECVF_RenderThreadSafe | ECVF_Default
);

constexpr uint32 INVALID_BONE_OFFSET = ~uint32(0);

bool IsValidBoneOffset(const uint32 BoneOffset)
{
	return BoneOffset != INVALID_BONE_OFFSET;
}

bool IsDynamicWindOnAsyncCompute()
{
	return GSupportsEfficientAsyncCompute && CVarDynamicWindPreferAsyncCompute.GetValueOnAnyThread();
}

}

class FDynamicWindEvalCS : public FGlobalShader
{
public:
	static constexpr uint32 BonesPerGroup = 64u;

private:
	DECLARE_GLOBAL_SHADER(FDynamicWindEvalCS);
	SHADER_USE_PARAMETER_STRUCT(FDynamicWindEvalCS, FGlobalShader);

	class FUseSinFunctionDim : SHADER_PERMUTATION_BOOL("USE_SIN_WIND");
	using FPermutationDomain = TShaderPermutationDomain<FUseSinFunctionDim>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);

		OutEnvironment.CompilerFlags.Add(CFLAG_WarningsAsErrors);
		OutEnvironment.CompilerFlags.Add(CFLAG_HLSL2021);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);

		OutEnvironment.SetDefine(TEXT("BONES_PER_GROUP"), BonesPerGroup);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FDynamicWindSkeletonData>, SkeletonBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FDynamicWindBlockHeader>, HeaderBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BoneDataBuffer)
		SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, BoneMapBuffer)
		SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_UAV(TransformBuffer)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, WindTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, WindSampler)
		SHADER_PARAMETER(float, Time)
		SHADER_PARAMETER(float, DeltaTime)
		SHADER_PARAMETER(float, WindSpeed)
		SHADER_PARAMETER(float, WindAmplitude)
		SHADER_PARAMETER(FVector3f, WindDirection)
		SHADER_PARAMETER(FVector4f, Debug)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FDynamicWindEvalCS, "/Plugin/DynamicWind/DynamicWindEval.usf", "WindEvalCS", SF_Compute);

TMap<const FScene*, FDynamicWindTransformProvider*> FDynamicWindTransformProvider::SceneProviderMap;

FDynamicWindTransformProvider* FDynamicWindTransformProvider::FindForScene(const FSceneInterface* InScene)
{
	if (const FScene* RenderScene = InScene ? InScene->GetRenderScene() : nullptr)
	{
		if (auto* Found = SceneProviderMap.Find(RenderScene))
		{
			return *Found;
		}
	}
	return nullptr;
}

FDynamicWindTransformProvider::FDynamicWindTransformProvider(FScene& InScene)
: Scene(InScene)
, BoneDataBuffer(1, TEXT("DynamicWind.BoneDataBuffer"))
{}

FDynamicWindTransformProvider::~FDynamicWindTransformProvider()
{	
	check(!bIsRegistered);
}

bool FDynamicWindTransformProvider::Register()
{
	check(!bIsRegistered);

	if (auto TransformProvider = Scene.GetExtensionPtr<FSkinningTransformProvider>())
	{
		// Register GPU dynamic wind transform provider
		TransformProvider->RegisterProvider(
			GDynamicWindTransformProviderId,
			FSkinningTransformProvider::FOnProvideTransforms::CreateRaw(this, &FDynamicWindTransformProvider::ProvideTransforms),
			true /* Use skeleton batching */
		);

		SceneProviderMap.Add(&Scene, this);

		bIsRegistered = true;
		return true;
	}

	return false;
}

bool FDynamicWindTransformProvider::Unregister()
{
	if (bIsRegistered)
	{
		if (auto TransformProvider = Scene.GetExtensionPtr<FSkinningTransformProvider>())
		{
			TransformProvider->UnregisterProvider(DynamicWindTransformProviderId);
		}

		SceneProviderMap.Remove(&Scene);

		bIsRegistered = false;
		return true;
	}

	return false;
}

void FDynamicWindTransformProvider::RegisterProxy(const FSkinningSceneExtensionProxy* ExtensionProxy, const UDynamicWindSkeletalData* InSkeletalData)
{
	if (!bIsRegistered)
	{
		UE_LOGF(LogDynamicWind, Verbose, "Trying to register proxy (SkinnedAsset=%ls), when the provider is not registered.", *GetNameSafe(ExtensionProxy->GetSkinnedAsset()));
		return;
	}

	const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ExtensionProxy->GetSkinnedAsset());
	if (SkeletalMesh == nullptr)
	{
		return;
	}

	const USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	const uint32 NumBonesInSkeleton = RefSkeleton.GetNum();

	const UDynamicWindSkeletalData* Data = InSkeletalData;
	if (Data == nullptr)
	{
		// No user data, don't register the skeleton. The wind simulation CS will provide identity
		return;
	}

	const uint64 UserDataHash = Data->GetSkeletalDataHash();

	const bool bUsesBoneMap = ExtensionProxy->GetBoneTransformStorageMode() == EBoneTransformStorageMode::BoneMap;
	FSkeletonEntry& SkeletonEntry = SkeletonLookup.FindOrAdd(FSkeletonKey{ Skeleton->GetGuid(), bUsesBoneMap });

	++SkeletonEntry.ReferenceCount;
	if (SkeletonEntry.ReferenceCount == 1)
	{
		SkeletonEntry.Data.BoneDataOffset = BoneDataAllocator.Allocate(NumBonesInSkeleton);
		SkeletonEntry.NumBones = NumBonesInSkeleton;
	#if DYNAMIC_WIND_DEBUG_SKELETON_NAMES
		SkeletonEntry.SkeletonName = Skeleton->GetFName();
	#endif
	}
	else
	{
		check(SkeletonEntry.Data.BoneDataOffset != INDEX_NONE);
		check(SkeletonEntry.NumBones == NumBonesInSkeleton);
		if (SkeletonEntry.UserDataHash == UserDataHash)
		{
			// no upload necessary
			return;
		}
	}

	if (!BoneDataUploader.IsValid())
	{
		BoneDataUploader = MakeUnique<FBoneDataUploader>();
	}

	const FBoxSphereBounds Bounds = SkeletalMesh->GetBounds();

	SkeletonEntry.UserDataHash = UserDataHash;
	SkeletonEntry.Data.PackedSimGroupAndGroundCover = 0;
	SetMaxSimulationGroupIndex(SkeletonEntry.Data, uint32(FMath::Max(Data->SimulationGroups.Num() - 1, 0)));
	SetIsGroundCover(SkeletonEntry.Data, Data->bIsGroundCover ? 1u : 0u);
	SkeletonEntry.Data.SkeletonHeight = Bounds.BoxExtent.Z + Bounds.Origin.Z;
	SkeletonEntry.Data.GustAttenuation = Data->GustAttenuation;

	TArrayView<FDynamicWindBoneData> BoneData = BoneDataUploader->AddMultiple_GetRef(SkeletonEntry.Data.BoneDataOffset, NumBonesInSkeleton);
	for (uint32 BoneIndex = 0; BoneIndex < NumBonesInSkeleton; ++BoneIndex)
	{
		const uint32 ParentBoneIndex = FMath::Min<uint32>(RefSkeleton.GetParentIndex(BoneIndex), DYNAMIC_WIND_INVALID_BONE_INDEX);			
		const FTransform AbsoluteTransform = RefSkeleton.GetBoneAbsoluteTransform(BoneIndex);
		const FQuat AbsoluteRotation = AbsoluteTransform.GetRotation();
		const FVector3f BindPosePosition = FVector3f(AbsoluteTransform.GetLocation());
		const FVector4f BindPoseRotation = FVector4f(
			AbsoluteRotation.X,
			AbsoluteRotation.Y,
			AbsoluteRotation.Z,
			AbsoluteRotation.W
		);

		BoneData[BoneIndex] = FDynamicWindBoneData
		{
			.ParentBoneIndex = ParentBoneIndex,
			.Influence = Data->GetBoneInfluence(BoneIndex),
			.SimulationGroupIndex = Data->GetBoneSimulationGroupIndex(BoneIndex),
			.BoneChainLength = Data->GetBoneChainLength(BoneIndex),
			.BindPoseRotation = BindPoseRotation,
			.BindPosePosition = BindPosePosition,
			.bIsTrunkBone = Data->IsTrunkBone(BoneIndex) ? 1u : 0u,
		};
	}
}

void FDynamicWindTransformProvider::UnregisterProxy(const FSkinningSceneExtensionProxy* ExtensionProxy)
{
	if (!bIsRegistered)
	{
		UE_LOGF(LogDynamicWind, Verbose, "Trying to unregister proxy (SkinnedAsset=%ls), when the provider is not registered.", *GetNameSafe(ExtensionProxy->GetSkinnedAsset()));
		return;
	}

	const USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(ExtensionProxy->GetSkinnedAsset());
	if (!SkeletalMesh)
	{
		return;
	}

	const USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
	if (!Skeleton)
	{
		return;
	}

	const bool bUsesBoneMap = ExtensionProxy->GetBoneTransformStorageMode() == EBoneTransformStorageMode::BoneMap;
	auto EntryID = SkeletonLookup.FindId(FSkeletonKey{ Skeleton->GetGuid(), bUsesBoneMap });

	if (!SkeletonLookup.IsValidId(EntryID))
	{
		return;
	}

	FSkeletonEntry& SkeletonEntry = SkeletonLookup.Get(EntryID).Value;
	--SkeletonEntry.ReferenceCount;
	if (SkeletonEntry.ReferenceCount == 0)
	{
		// Last proxy out, remove the whole entry
		BoneDataAllocator.Free(SkeletonEntry.Data.BoneDataOffset, SkeletonEntry.NumBones);
		SkeletonLookup.Remove(EntryID);
	}
}

int32 FDynamicWindTransformProvider::UpdateParameters(const FDynamicWindParameters& Parameters)
{
	WindParameters = Parameters;
	return INDEX_NONE;
}

float FDynamicWindTransformProvider::GetBlendedWindAmplitude() const
{
	return BlendedWindAmplitude;
}

void FDynamicWindTransformProvider::ProvideTransforms(FSkinningTransformProvider::FProviderContext& Context)
{
	const bool bHasAlreadySimulatedThisFrame = LastSimulatedFrameNumber == Scene.GetFrameNumber();
	
	// There are multiple broadcasts from SkinningSceneExtension, for primary view as well virtual texture updates.
	// But we need to run the simulation only once.
	if (bHasAlreadySimulatedThisFrame || SkeletonLookup.IsEmpty())
	{
		return;
	}
	
	const uint32 BonesPerGroup = FDynamicWindEvalCS::BonesPerGroup;
	uint32 BlockCount = 0;
	for (const FSkinningTransformProvider::FProviderIndirection Indirection : Context.Indirections)
	{
		const FSkeletonBatch& SkeletonBatch = Context.SkeletonBatches[Indirection.Index];
		const uint32 TransformCount = SkeletonBatch.MaxBoneTransforms;
		BlockCount += FMath::DivideAndRoundUp(TransformCount, BonesPerGroup) * SkeletonBatch.UniqueAnimationCount;
	}

	if (BlockCount == 0)
	{
		// Nothing to simulate
		return;
	}

	FRDGBuilder& GraphBuilder = Context.GraphBuilder;

	// Create the skeleton data and block header buffers
	const int32 SkeletonCount = SkeletonLookup.Num();
	UE::HLSL::FDynamicWindSkeletonData* SkeletonData = GraphBuilder.AllocPODArray<UE::HLSL::FDynamicWindSkeletonData>(SkeletonCount);
	UE::HLSL::FDynamicWindBlockHeader* BlockHeaders = GraphBuilder.AllocPODArray<UE::HLSL::FDynamicWindBlockHeader>(BlockCount);

	GraphBuilder.AddSetupTask(
		[
			BlockHeaders,
			SkeletonData,
			SkeletonLookup = TMap<FSkeletonKey, FSkeletonEntry, SceneRenderingSetAllocator>(SkeletonLookup), // Copy for task
			Indirections = Context.Indirections,
			SkeletonBatches = Context.SkeletonBatches
		]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FDynamicWindTransformProvider::ProvideTransforms);

			int32 SkeletonIndex = 0;
			for (const auto& [_, SkeletonEntry] : SkeletonLookup)
			{
				SkeletonData[SkeletonIndex] = SkeletonEntry.Data;
				++SkeletonIndex;
			}

			uint32 BlockWrite = 0;
			for (const FSkinningTransformProvider::FProviderIndirection Indirection : Indirections)
			{
				const FSkeletonBatch& SkinnedAnimationBatch = SkeletonBatches[Indirection.Index];

				const uint32 TransformCount = SkinnedAnimationBatch.MaxBoneTransforms;
				const uint32 UniqueAnimationCount = SkinnedAnimationBatch.UniqueAnimationCount;

				const uint32 FullBlockCount = TransformCount / BonesPerGroup;
				const uint32 PartialTransformCount = TransformCount - (FullBlockCount * BonesPerGroup);

				uint32 DstTransformOffset = Indirection.CurrentTransformOffset;

				const bool bUsesBoneMap = Indirection.BoneMapOffset != INDEX_NONE;
				FSetElementId SkeletonId = SkeletonLookup.FindId(FSkeletonKey{ SkinnedAnimationBatch.SkeletonGuid, bUsesBoneMap });

				const uint32 BoneMapBufferOffset = Indirection.BoneMapOffset;

				for (uint32 DirectionalityIndex = 0; DirectionalityIndex < UniqueAnimationCount; ++DirectionalityIndex)
				{
					uint32 BoneOffset = 0;

					for (uint32 BlockIndex = 0; BlockIndex < FullBlockCount; ++BlockIndex)
					{
						FDynamicWindBlockHeader& Header = BlockHeaders[BlockWrite];
						Header.SkeletonDataIndex = SkeletonId.AsInteger(); // NOTE: Can be INDEX_NONE
						Header.BlockDstTransformOffset = DstTransformOffset;
						Header.PackedNumBonesAndOffset = 0;
						SetBlockNumBones(Header, BonesPerGroup);
						SetBlockBoneOffset(Header, BoneOffset);
						Header.PackedDirectionalityAndTransformCount = 0;
						SetBlockDirectionalityIndex(Header, DirectionalityIndex);
						SetTotalTransformCount(Header, TransformCount);
						Header.BoneMapBufferOffset = BoneMapBufferOffset;
						
						BoneOffset += BonesPerGroup;
						DstTransformOffset += (BonesPerGroup * sizeof(FCompressedBoneTransform));
						++BlockWrite;
					}

					if (PartialTransformCount > 0)
					{
						FDynamicWindBlockHeader& Header = BlockHeaders[BlockWrite];
						Header.SkeletonDataIndex = SkeletonId.AsInteger(); // NOTE: Can be INDEX_NONE
						Header.BlockDstTransformOffset = DstTransformOffset;
						Header.PackedNumBonesAndOffset = 0;
						SetBlockNumBones(Header, PartialTransformCount);
						SetBlockBoneOffset(Header, BoneOffset);
						Header.PackedDirectionalityAndTransformCount = 0;
						SetBlockDirectionalityIndex(Header, DirectionalityIndex);
						SetTotalTransformCount(Header, TransformCount);
						Header.BoneMapBufferOffset = BoneMapBufferOffset;

						BoneOffset += PartialTransformCount;
						DstTransformOffset += (PartialTransformCount * sizeof(FCompressedBoneTransform));
						++BlockWrite;
					}

					// Advance past all the previous transforms as well
					DstTransformOffset += (sizeof(FCompressedBoneTransform) * TransformCount);
				}
			}
		},
		UE::Tasks::ETaskPriority::High
	);

	FRDGBufferRef SkeletonBuffer = CreateStructuredBuffer(
		GraphBuilder,
		TEXT("DynamicWind.SkeletonData"),
		MakeConstArrayView(SkeletonData, SkeletonCount),
		ERDGInitialDataFlags::NoCopy // The buffer data is allocated above on the RDG timeline
	);

	FRDGBufferRef BlockHeaderBuffer = CreateStructuredBuffer(
		GraphBuilder,
		TEXT("DynamicWind.ScatterHeaders"),
		MakeConstArrayView(BlockHeaders, BlockCount),
		ERDGInitialDataFlags::NoCopy // The buffer data is allocated above on the RDG timeline
	);

	FRDGBufferRef BoneDataBufferRDG = nullptr; 
	if (BoneDataUploader.IsValid())
	{
		BoneDataBufferRDG = BoneDataUploader->ResizeAndUploadTo(
			GraphBuilder,
			BoneDataBuffer,
			BoneDataAllocator.GetMaxSize()
		);
		BoneDataUploader = nullptr;
	}
	else
	{
		BoneDataBufferRDG = BoneDataBuffer.Register(GraphBuilder);
	}

	FTextureRHIRef WindTextureRHI = nullptr;
	FSamplerStateRHIRef WindSampler = nullptr;
	if (WindParameters.WindTexture)
	{
		if (FTextureResource* WindTextureResource = WindParameters.WindTexture->GetResource())
		{
			WindTextureRHI = WindTextureResource->GetTextureRHI();
			WindSampler = WindTextureResource->SamplerStateRHI;
		}
	}
	if (WindTextureRHI == nullptr)
	{
		WindTextureRHI = GWhiteTexture->TextureRHI.GetReference();
	}

	if (WindSampler == nullptr)
	{
		WindSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
	}

	auto WindTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(WindTextureRHI, TEXT("DynamicWind.WindTexture")));

	const float WindSpeedOverride = CVarDynamicWindOverrideSpeed.GetValueOnAnyThread();
	const float WindSpeed = WindSpeedOverride < 0.0f ? WindParameters.WindSpeed : WindSpeedOverride;

	const float WindAmplitudeOverride = CVarDynamicWindOverrideAmplitude.GetValueOnAnyThread();
	if (!FMath::IsNearlyEqual(WindAmplitudeOverride, BlendedWindAmplitude, 1.e-2f))
	{
		const float RateOfChange = CVarDynamicWindOverrideRateOfChange.GetValueOnAnyThread();
		BlendedWindAmplitude += FMath::Sign(WindAmplitudeOverride - BlendedWindAmplitude) * RateOfChange * Scene.GetWorld()->GetDeltaSeconds();
	}
	else
	{
		BlendedWindAmplitude = WindAmplitudeOverride;
	}
	
	const float WindAmplitude = BlendedWindAmplitude < 0.0f ? WindParameters.WindAmplitude : BlendedWindAmplitude;

	{
		RDG_EVENT_SCOPE_STAT(GraphBuilder, DynamicWind, "DynamicWind");

		FDynamicWindEvalCS::FParameters* PassParameters = GraphBuilder.AllocParameters<FDynamicWindEvalCS::FParameters>();
		PassParameters->SkeletonBuffer = GraphBuilder.CreateSRV(SkeletonBuffer);
		PassParameters->HeaderBuffer = GraphBuilder.CreateSRV(BlockHeaderBuffer);
		PassParameters->BoneDataBuffer = GraphBuilder.CreateSRV(BoneDataBufferRDG);
		PassParameters->BoneMapBuffer = Context.BoneMapBufferSRV;
		PassParameters->TransformBuffer = GetCompressedBoneTransformUAV(GraphBuilder, Context.TransformBuffer);
		PassParameters->WindTexture = GraphBuilder.CreateSRV(WindTextureRef);
		PassParameters->WindSampler = WindSampler;
		PassParameters->Time = Scene.GetWorld()->GetTimeSeconds();
		PassParameters->DeltaTime = Scene.GetWorld()->GetDeltaSeconds();
		PassParameters->WindSpeed = WindSpeed;
		PassParameters->WindAmplitude = WindAmplitude;
		PassParameters->WindDirection = FVector3f(WindParameters.WindDirection);
		PassParameters->Debug = WindParameters.DebugModulation;

		FDynamicWindEvalCS::FPermutationDomain PermutationVectorCS;
		PermutationVectorCS.Set<FDynamicWindEvalCS::FUseSinFunctionDim>(CVarDynamicWindUseSine.GetValueOnAnyThread());

		auto ComputeShader = GetGlobalShaderMap(GMaxRHIFeatureLevel)->GetShader<FDynamicWindEvalCS>(PermutationVectorCS);
		FComputeShaderUtils::AddPass(
			GraphBuilder,
			RDG_EVENT_NAME("DynamicWind Simulate [BlockCount: %d]", BlockCount),
			IsDynamicWindOnAsyncCompute() ? ERDGPassFlags::AsyncCompute : ERDGPassFlags::Compute,
			ComputeShader,
			PassParameters,
			FIntVector(BlockCount, 1, 1)
		);
	}

	LastSimulatedFrameNumber = Scene.GetFrameNumber();
}
