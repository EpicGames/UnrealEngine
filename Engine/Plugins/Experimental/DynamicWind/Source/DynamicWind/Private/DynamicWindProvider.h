// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"
#include "Engine/SkeletalMesh.h"
#include "HLSLTypeAliases.h"
#include "Shared/DynamicWindCommon.ush"
#include "DynamicWindParameters.h"
#include "Skinning/SkinningTransformProvider.h"
#include "SkinningDefinitions.h"
#include "Animation/TransformProviderData.h"

class USkeleton;
class FSceneInterface;
class UDynamicWindSkeletalData;

#define DYNAMIC_WIND_TRANSFORM_PROVIDER_GUID 0xDFD4874B, 0xEE57466D, 0x883D6419, 0xAFE99EAC
#define DYNAMIC_WIND_DEBUG_SKELETON_NAMES (UE_BUILD_DEBUG | UE_BUILD_DEVELOPMENT)

class FDynamicWindTransformProvider
{
public:
	explicit FDynamicWindTransformProvider(FScene& InScene);
	FDynamicWindTransformProvider() = delete;
	~FDynamicWindTransformProvider();

	bool Register();
	bool Unregister();

	static FDynamicWindTransformProvider* FindForScene(const FSceneInterface* Scene);

	void RegisterProxy(const FSkinningSceneExtensionProxy* ExtensionProxy, const UDynamicWindSkeletalData* InSkeletalData);
	void UnregisterProxy(const FSkinningSceneExtensionProxy* ExtensionProxy);
	int32 UpdateParameters(const FDynamicWindParameters& Parameters);

	float GetBlendedWindAmplitude() const;

private:
	using FBoneDataBuffer = TPersistentByteAddressBuffer<FDynamicWindBoneData>;
	using FBoneDataUploader = TByteAddressBufferScatterUploader<FDynamicWindBoneData>;

	struct FSkeletonKey
	{
		FGuid SkeletonGuid;
		bool bUsesBoneMap = false;

		bool operator==(const FSkeletonKey& Other) const
		{
			return SkeletonGuid == Other.SkeletonGuid && bUsesBoneMap == Other.bUsesBoneMap;
		}

		friend uint32 GetTypeHash(const FSkeletonKey& Key)
		{
			return HashCombine(GetTypeHash(Key.SkeletonGuid), GetTypeHash(Key.bUsesBoneMap));
		}
	};

	struct FSkeletonEntry
	{	
		uint64 UserDataHash = 0;
		uint32 NumBones = 0;
		uint32 ReferenceCount = 0;
	#if DYNAMIC_WIND_DEBUG_SKELETON_NAMES
		FName SkeletonName;
	#endif
		
		FDynamicWindSkeletonData Data;
	};

	void ProvideTransforms(FSkinningTransformProvider::FProviderContext& Context);

private:
	FScene& Scene;

	TMap<FSkeletonKey, FSkeletonEntry> SkeletonLookup;
	FSpanAllocator BoneDataAllocator;
	TUniquePtr<FBoneDataUploader> BoneDataUploader;
	FBoneDataBuffer BoneDataBuffer;

	FDynamicWindParameters WindParameters = {};

	uint64 LastSimulatedFrameNumber = 0u;

	float BlendedWindAmplitude = -1.0f;

	bool bIsRegistered = false;

	static TMap<const FScene*, FDynamicWindTransformProvider*> SceneProviderMap;
};