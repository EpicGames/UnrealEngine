// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicWindSkeletalData.h"
#include "Serialization/MemoryHasher.h"
#include "Hash/xxhash.h"

int32 UDynamicWindSkeletalData::GetBoneSimulationGroupIndex(const int32 BoneIndex) const
{
	for (const auto& [SimulationGroupIndex, BoneIndices] : SimulationGroupBones)
	{
		if (BoneIndices.Contains(BoneIndex))
		{
			return SimulationGroupIndex;
		}
	}

	return INDEX_NONE;
}

float UDynamicWindSkeletalData::GetBoneInfluence(const int32 BoneIndex) const
{
	const FDynamicWindExtraBoneData* ExtraBoneData = ExtraBonesData.Find(BoneIndex);
	if (ExtraBoneData == nullptr)
	{
		return 0.0f;
	}
	
	const int32 SimulationGroupIndex = GetBoneSimulationGroupIndex(BoneIndex);
	if (!SimulationGroups.IsValidIndex(SimulationGroupIndex))
	{
		return 1.0f; // The user may have deleted this SimulationGroup
	}

	const FDynamicWindSimulationGroupData& SimulationGroup = SimulationGroups[SimulationGroupIndex];
	if (!SimulationGroup.bUseDualInfluence)
	{
		return SimulationGroup.Influence;
	}
	
	float InfluenceBlend = 0.0f;
	if (const FDynamicWindBoneChainData* BoneChain = BoneChains.Find(ExtraBoneData->BoneChainOriginBoneIndex))
	{		
		InfluenceBlend = BoneChain->NumBones > 1 ? ExtraBoneData->IndexInBoneChain / (BoneChain->NumBones - 1.0f) : 0.0f;

		// Formula for a bend toward zero: y = x ^ (1 / (1 - shift))
		// Shift is biased to prevent division by zero
		InfluenceBlend = FMath::Pow(InfluenceBlend, 1.0f / (1.0f - FMath::Min(SimulationGroup.ShiftTop, 0.99f)));
	}
	
	return FMath::Lerp(SimulationGroup.MinInfluence, SimulationGroup.MaxInfluence, InfluenceBlend);
}

bool UDynamicWindSkeletalData::IsTrunkBone(const int32 BoneIndex) const
{
	if (!ExtraBonesData.Contains(BoneIndex))
	{
		return false;
	}
	
	const int32 SimulationGroupIndex = GetBoneSimulationGroupIndex(BoneIndex);
	return SimulationGroups.IsValidIndex(SimulationGroupIndex) ? SimulationGroups[SimulationGroupIndex].bIsTrunkGroup : false;
}

float UDynamicWindSkeletalData::GetBoneChainLength(const int32 BoneIndex) const
{
	const FDynamicWindExtraBoneData* ExtraBoneData = ExtraBonesData.Find(BoneIndex);
	if (ExtraBoneData == nullptr)
	{
		return 0.0f;
	}

	if (const FDynamicWindBoneChainData* BoneChain = BoneChains.Find(ExtraBoneData->BoneChainOriginBoneIndex))
	{
		return BoneChain->ChainLength;
	}
	
	return 0.0f;
}

void UDynamicWindSkeletalData::RecalculateSkeletalDataHash()
{
	TMemoryHasher<FXxHash64Builder, FXxHash64> Ar;
	Ar.SetIsPersistent(true);
	
	Serialize(Ar);
		
	SkeletalDataHash = Ar.Finalize().Hash;
}

void UDynamicWindSkeletalData::PostLoad()
{
	Super::PostLoad();
	RecalculateSkeletalDataHash();
}

#if WITH_EDITOR

void UDynamicWindSkeletalData::PostEditChangeProperty(FPropertyChangedEvent& Event)
{
	Super::PostEditChangeProperty(Event);
	RecalculateSkeletalDataHash();
}

#endif // WITH_EDITOR