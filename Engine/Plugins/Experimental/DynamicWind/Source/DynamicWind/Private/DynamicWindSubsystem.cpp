// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicWindSubsystem.h"
#include "Animation/Skeleton.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "Engine/SkinnedAsset.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "GlobalRenderResources.h"
#include "GlobalShader.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphResources.h"
#include "RenderGraphUtils.h"
#include "SceneView.h"
#include "SceneInterface.h"
#include "ScenePrivate.h"
#include "DynamicWindSkeletalData.h"
#include "DynamicWindProvider.h"

namespace
{

TAutoConsoleVariable<bool> CVarDynamicWindEnable(
	TEXT("DynamicWind.Enable"),
	true,
	TEXT("Whether or not to enable the dynamic wind subsystem"),
	ECVF_RenderThreadSafe | ECVF_ReadOnly
);

}

bool UDynamicWindSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return CVarDynamicWindEnable.GetValueOnAnyThread();
}

void UDynamicWindSubsystem::PostInitialize()
{
	Super::PostInitialize();

	if (UWorld* World = GetWorld())
	{
		CreateTransformProvider(*World);
	}

#if WITH_EDITOR
	OnPreRecreateScene = FWorldDelegates::OnPreRecreateScene.AddLambda(
		[this](UWorld* World)
		{
			if (World == GetWorld())
			{
				ReleaseTransformProvider();
			}
		}
	);
	OnPostRecreateScene = FWorldDelegates::OnPostRecreateScene.AddLambda(
		[this](UWorld* World)
		{
			if (World == GetWorld())
			{
				CreateTransformProvider(*World);
			}
		}
	);
#endif // WITH_EDITOR
}

void UDynamicWindSubsystem::PreDeinitialize()
{
	ReleaseTransformProvider();

#if WITH_EDITOR
	if (OnPreRecreateScene.IsValid())
	{
		FWorldDelegates::OnPreRecreateScene.Remove(OnPreRecreateScene);
		OnPreRecreateScene.Reset();
	}
	if (OnPostRecreateScene.IsValid())
	{
		FWorldDelegates::OnPostRecreateScene.Remove(OnPostRecreateScene);
		OnPostRecreateScene.Reset();
	}
#endif // WITH_EDITOR

	Super::PreDeinitialize();
}

float UDynamicWindSubsystem::GetBlendedWindAmplitude() const
{
	if (TransformProvider.IsValid())
	{
		return TransformProvider->GetBlendedWindAmplitude();
	}

	return -1.0f;
}

void UDynamicWindSubsystem::UpdateWindParameters(const FDynamicWindParameters& Parameters)
{
	if (TransformProvider.IsValid())
	{
		ENQUEUE_RENDER_COMMAND(UpdateDynamicWindParameters)
		(
			[WindProvider=TransformProvider, WindParameters=Parameters](FRHICommandListImmediate&) mutable
			{
				WindProvider->UpdateParameters(WindParameters);
			}
		);
	}
}

void UDynamicWindSubsystem::CreateTransformProvider(UWorld& World)
{
	if (FSceneInterface* Scene = World.Scene)
	{
		if (FScene* RenderScene = Scene->GetRenderScene())
		{
			TransformProvider = MakeShared<FDynamicWindTransformProvider, ESPMode::ThreadSafe>(*RenderScene);

			// Transform provider must be registered on the render thread
			ENQUEUE_RENDER_COMMAND(RegisterDynamicWindProvider)
			(
				[ProviderToRegister = TransformProvider](FRHICommandListImmediate&)
				{
					ProviderToRegister->Register();
				}
			);
		}
	}
}

void UDynamicWindSubsystem::ReleaseTransformProvider()
{
	if (TransformProvider.IsValid())
	{
		// Transform provider must be unregistered and destroyed on the render thread
		ENQUEUE_RENDER_COMMAND(UnregisterAndDestroyDynamicWindProvider)
		(
			[ProviderToDestroy = MoveTemp(TransformProvider)](FRHICommandListImmediate&) mutable
			{
				ProviderToDestroy->Unregister();
				ProviderToDestroy.Reset(); // should be the last reference out
			}
		);
	}
}