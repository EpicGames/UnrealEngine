// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/TransformProviderData.h"

#include "DynamicWindData.generated.h"

UCLASS(hidecategories=Object, MinimalAPI, BlueprintType)
class UDynamicWindData : public UTransformProviderData
{
	GENERATED_BODY()

public:
	virtual bool IsEnabled() const override;
	virtual bool IsGpuOnly() const override { return true; }
	virtual const FGuid& GetTransformProviderID() const override;
	virtual uint32 GetUniqueAnimationCount() const override;
	virtual bool UsesSkeletonBatching() const override;
	virtual bool HasAnimationBounds() const override;
	virtual bool GetAnimationBounds(uint32 AnimationIndex, FRenderBounds& OutBounds) const override;
	virtual uint32 GetSkinningDataOffset(int32 InstanceIndex, const FTransform& ComponentTransform, const FSkinnedMeshInstanceData& InstanceData) const override;
	virtual FTransformProviderRenderProxy* CreateRenderProxy(FInstancedSkinningSceneExtensionProxy* ExtensionProxy) const override;
};

class FDynamicWindDataProxy : public FTransformProviderRenderProxy
{
	friend class UDynamicWindData;

public:
	DYNAMICWIND_API FDynamicWindDataProxy(const UDynamicWindData* BankData, FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy);
	DYNAMICWIND_API virtual ~FDynamicWindDataProxy();
	DYNAMICWIND_API virtual void CreateRenderThreadResources(FSceneInterface& Scene, FRHICommandListBase& RHICmdList) override;
	DYNAMICWIND_API virtual void DestroyRenderThreadResources() override;

private:
	FInstancedSkinningSceneExtensionProxy* SceneExtensionProxy = nullptr;
	TObjectPtr<const class UDynamicWindSkeletalData> SkeletalData = nullptr;
	class FDynamicWindTransformProvider* TransformProvider = nullptr;
};
