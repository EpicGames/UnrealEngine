// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicWindParameters.generated.h"

class UTexture;

USTRUCT(BlueprintType)
struct FDynamicWindParameters
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DynamicWind")
	FVector SimulationCenter = FVector::Zero();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DynamicWind")
	float SimulationExtents = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DynamicWind")
	FVector WindDirection = FVector::ForwardVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DynamicWind")
	float WindSpeed = 15.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DynamicWind")
	float WindAmplitude = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DynamicWind")
	TObjectPtr<UTexture> WindTexture = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DynamicWind")
	FVector4f DebugModulation = FVector4f::Zero();
};
