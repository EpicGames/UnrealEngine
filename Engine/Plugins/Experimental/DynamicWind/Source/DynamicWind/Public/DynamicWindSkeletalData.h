// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/AssetUserData.h"

#include "DynamicWindSkeletalData.generated.h"

USTRUCT()
struct FDynamicWindSimulationGroupData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Skeletal Data")
	bool bUseDualInfluence = false;
	
	UPROPERTY(EditAnywhere, Category = "Skeletal Data", meta=(EditCondition = "!bUseDualInfluence", EditConditionHides, ClampMin=0.0f, ClampMax=1.0f))
	float Influence = 0.0f;
	
	UPROPERTY(EditAnywhere, Category = "Skeletal Data", meta=(EditCondition = "bUseDualInfluence", EditConditionHides, ClampMin=0.0f, ClampMax=1.0f))
	float MinInfluence = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Skeletal Data", meta=(EditCondition = "bUseDualInfluence", EditConditionHides, ClampMin=0.0f, ClampMax=1.0f))
	float MaxInfluence = 0.0f;
	
	UPROPERTY(EditAnywhere, Category = "Skeletal Data", meta=(EditCondition = "bUseDualInfluence", EditConditionHides, ClampMin=0.0f, ClampMax=1.0f))
	float ShiftTop = 0.0f;
	
	UPROPERTY(EditAnywhere, Category = "Skeletal Data", meta=(DisplayName = "Is Trunk?"))
	bool bIsTrunkGroup = false;
};

USTRUCT()
struct FDynamicWindSimulationGroupBoneLookup
{
	GENERATED_BODY()

	UPROPERTY()
	int32 SimulationGroupIndex = INDEX_NONE;

	UPROPERTY()
	TSet<int32> BoneIndices;
};

USTRUCT()
struct FDynamicWindBoneChainData
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 NumBones = 0;

	UPROPERTY()
	float ChainLength = 0.0f;
};

USTRUCT()
struct FDynamicWindExtraBoneData
{
	GENERATED_BODY()

	UPROPERTY()
	int32 BoneChainOriginBoneIndex = INDEX_NONE;
	
	UPROPERTY()
	int32 IndexInBoneChain = 0;
};

UCLASS()
class DYNAMICWIND_API UDynamicWindSkeletalData : public UAssetUserData
{
	GENERATED_BODY()

public:
	/** When true, the SkeletalMesh object will be registered with the DynamicWind system. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Data")
	bool bIsEnabled = false;

	/** If true, marks this object as ground cover, and won't attenuate wind based on bone height. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Data")
	bool bIsGroundCover = false;

	/** Uniformly attenuates the influence of wind on trunk simulation groups. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Data")
	float GustAttenuation = 0.0f;
	
	UPROPERTY(EditAnywhere, Category = "Skeletal Data")
	TArray<FDynamicWindSimulationGroupData> SimulationGroups;
	
	UPROPERTY()
	TArray<FDynamicWindSimulationGroupBoneLookup> SimulationGroupBones;

	UPROPERTY()
	TMap<int32, FDynamicWindBoneChainData> BoneChains;

	UPROPERTY()
	TMap<int32, FDynamicWindExtraBoneData> ExtraBonesData;

public:
	int32 GetBoneSimulationGroupIndex(int32 BoneIndex) const;
	float GetBoneInfluence(int32 BoneIndex) const;
	bool IsTrunkBone(int32 BoneIndex) const;
	float GetBoneChainLength(int32 BoneIndex) const;
	void RecalculateSkeletalDataHash();
	uint64 GetSkeletalDataHash() const { return SkeletalDataHash; }
	
public:
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& Event) override;
#endif

private:	
	uint64 SkeletalDataHash = 0ull;
};
