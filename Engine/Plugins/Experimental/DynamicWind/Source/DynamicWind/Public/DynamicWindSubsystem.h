// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "SpanAllocator.h"
#include "Matrix3x4.h"
#include "Containers/Map.h"
#include "DynamicWindLog.h"
#include "DynamicWindParameters.h"
#include "DynamicWindSubsystem.generated.h"

class USkeleton;
class UInstancedSkinnedMeshComponent;
class FDynamicWindTransformProvider;

UCLASS(BlueprintType, ClassGroup = (Rendering, Common))
class UDynamicWindSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	DYNAMICWIND_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;

	DYNAMICWIND_API virtual void PostInitialize() override;
	DYNAMICWIND_API virtual void PreDeinitialize() override;

public:
	UFUNCTION(BlueprintCallable, Category = "DynamicWind")
	DYNAMICWIND_API float GetBlendedWindAmplitude() const;

	UFUNCTION(BlueprintCallable, Category = "DynamicWind")
	DYNAMICWIND_API void UpdateWindParameters(const FDynamicWindParameters& Parameters);

private:
	void CreateTransformProvider(UWorld& World);
	void ReleaseTransformProvider();

	using FTransformProviderPtr = TSharedPtr<FDynamicWindTransformProvider, ESPMode::ThreadSafe>;

	FTransformProviderPtr TransformProvider;
#if WITH_EDITORONLY_DATA
	FDelegateHandle OnPreRecreateScene;
	FDelegateHandle OnPostRecreateScene;
#endif
};
