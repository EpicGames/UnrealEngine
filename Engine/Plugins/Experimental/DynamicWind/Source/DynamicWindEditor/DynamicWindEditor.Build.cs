// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class DynamicWindEditor : ModuleRules
	{
		public DynamicWindEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.AddRange(
				new string[] {
					Path.Combine(GetModuleDirectory("Renderer"), "Private"),
					Path.Combine(PluginDirectory, "Shaders")
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"AssetTools",
					"Core",
					"CoreUObject",
					"EditorFramework",
					"EditorStyle",
					"Engine",
					"PropertyEditor",
					"RHI",
					"Slate",
					"Slate",
					"SlateCore",
					"TargetPlatform",
					"UnrealEd",
					"SourceControl",
					"MeshDescription",
					"ImageCore",
					"StaticMeshDescription",
					"SkeletalMeshDescription",
					"Json",
					"JsonUtilities"
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorSubsystem",
					"DynamicWind",
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"DerivedDataCache",
				}
			);
		}
	}
}
