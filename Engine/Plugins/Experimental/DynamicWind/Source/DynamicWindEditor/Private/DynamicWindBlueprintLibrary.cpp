// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicWindBlueprintLibrary.h"

#include "IAssetTools.h"
#include "ImageCore.h"
#include "IntVectorTypes.h"
#include "SkeletalMeshAttributes.h"
#include "SkeletalMeshTypes.h"
#include "StaticMeshAttributes.h"
#include "Animation/Skeleton.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Rendering/SkeletalMeshModel.h"
#include "DynamicWindSkeletalData.h"
#include "DynamicWindImportData.h"
#include "Misc/FileHelper.h"
#include "JsonObjectConverter.h"
#include "EditorFramework/AssetImportData.h"
#include "DesktopPlatformModule.h"
#include "Framework/Application/SlateApplication.h"

DEFINE_LOG_CATEGORY_STATIC(LogDynamicWindBlueprintLibrary, Log, All);

bool UDynamicWindBlueprintLibrary::ConvertPivotPainterTreeToSkeletalMesh(
	const UStaticMesh* TreeStaticMesh,
	UTexture2D* TreePivotPosTexture,
	int32 TreePivotUVIndex,
	USkeletalMesh* TargetSkeletalMesh,
	USkeleton* TargetSkeleton
	)
{
	if (!TreeStaticMesh)
	{
		UE_LOGF(LogDynamicWindBlueprintLibrary, Error, "No static mesh given");
		return false;
	}
	if (!TreePivotPosTexture)
	{
		UE_LOGF(LogDynamicWindBlueprintLibrary, Error, "No pivot pos texture given");
		return false;
	}
	if (!TargetSkeletalMesh)
	{
		UE_LOGF(LogDynamicWindBlueprintLibrary, Error, "No target skeletal mesh given");
		return false;
	}
	if (!TargetSkeleton)
	{
		UE_LOGF(LogDynamicWindBlueprintLibrary, Error, "No target skeleton given");
		return false;
	}
	if (TargetSkeletalMesh->GetLODNum() != 0 || TargetSkeletalMesh->GetSkeleton() || TargetSkeletalMesh->GetRefSkeleton().GetNum() != 0)
	{
		UE_LOGF(LogDynamicWindBlueprintLibrary, Error, "Target skeletal mesh is not empty");
		return false;
	}
	if (TargetSkeleton->GetReferenceSkeleton().GetNum() != 0)
	{
		UE_LOGF(LogDynamicWindBlueprintLibrary, Error, "Target skeleton is not empty");
		return false;
	}
	

	// Try hi-res mesh first, fall back to LOD 0.
	FMeshDescription MeshDescription; 
	if (!TreeStaticMesh->CloneHiResMeshDescription(MeshDescription))
	{
		if (!TreeStaticMesh->CloneMeshDescription(0, MeshDescription))
		{
			UE_LOGF(LogDynamicWindBlueprintLibrary, Error, "No valid geometry found on tree mesh");
			return false;
		}
	}

	FSkeletalMeshAttributes MeshAttributes(MeshDescription);

	constexpr bool bKeepExistingAttributes = true;
	MeshAttributes.Register(bKeepExistingAttributes);
	
	TVertexInstanceAttributesConstRef<FVector2f> UVCoords = MeshAttributes.GetVertexInstanceUVs();
	
	if (TreePivotUVIndex < 0 || TreePivotUVIndex >= UVCoords.GetNumChannels())
	{
		UE_LOGF(LogDynamicWindBlueprintLibrary, Error, "Invalid Pivot UV layer %d (Max layers: %d)", TreePivotUVIndex, UVCoords.GetNumChannels());
	}

	FImage PivotImage;
	TreePivotPosTexture->Source.GetMipImage(PivotImage, 0);

	// Ask about which pixels are used from the pivot texture. Then use that to build the skeleton.
	struct FPivot
	{
		FVector3f Pos;
		int32 Parent = INDEX_NONE;
		int32 BoneIndex = INDEX_NONE;
		bool bHasChildren = false;
		FName BoneName;
	};
	
	TArray<FTriangleID> InvalidTriangleIDs;
	TMap<int32, FPivot> Pivots;
	TMap<FVector2f, int32> UVToPivot;
	for (FTriangleID TriangleID: MeshDescription.Triangles().GetElementIDs())
	{
		TArrayView<const FVertexInstanceID> Corners = MeshDescription.GetTriangleVertexInstances(TriangleID);
		for (int32 Index = 0; Index < Corners.Num(); Index++)
		{
			FVector2f UV = UVCoords.Get(Corners[Index], TreePivotUVIndex);

			// Out-of-bound UV indicates root binding, we handle that when we do binding. 
			if (UV.X >= 0.0f && UV.X < 1.0f && UV.Y >= 0.0f && UV.Y < 1.0f)
			{
				FInt32Vector2 Pixel;
				Pixel.X = UV.X * PivotImage.SizeX;
				Pixel.Y = UV.Y * PivotImage.SizeY;

				const int32 PivotIndex = Pixel.X + Pixel.Y * PivotImage.SizeX;
				Pivots.Add(PivotIndex, {});
				UVToPivot.Add(UV, PivotIndex);
			}
			else
			{
				Pivots.Add(0, {});
				UVToPivot.Add(UV, 0);
			}
		}
	}

	// Crawl through the pivot texture, using the used pivots, to get position and parent indices.
	TArray<int32> PivotOrder;
	
	for (TPair<int32, FPivot>& Item: Pivots)
	{
		const int32 PivotIndex = Item.Key; 
		FPivot& Pivot = Item.Value;
		
		PivotOrder.Add(PivotIndex);

		// Pivot 0 is the root.
		if (PivotIndex == 0)
		{
			Pivot.Parent = INDEX_NONE;
			Pivot.Pos = FVector3f::ZeroVector;
		}
		else
		{
			const int32 X = PivotIndex % PivotImage.SizeX;
			const int32 Y = PivotIndex / PivotImage.SizeX;
			
			FLinearColor C = PivotImage.GetOnePixelLinear(X, Y);
			
			const uint32 FloatBits = *reinterpret_cast<uint32*>(&C.A);
			const uint32 Sign = ((FloatBits>>16)&0x8000);
			const uint32 Exponent  = ((static_cast<const int32>((FloatBits >> 23) & 0xff)-127+15) << 10);
			const uint32 Mantissa = ((FloatBits>>13)&0x3ff);
			const uint32 Bits = (Sign | Exponent | Mantissa);
			const uint32 ParentIndex = Bits - 1024;

			Pivot.Pos = FVector3f(C.R, C.G, C.B);
			Pivot.Parent = ParentIndex == PivotIndex ? 0 : ParentIndex;
		}
	}

	// If nothing was explicitly bound to the root, add the root pivot. 
	if(!Pivots.Contains(0))
	{
		FPivot RootPivot;
		RootPivot.Parent = INDEX_NONE;
		RootPivot.Pos = FVector3f::ZeroVector;
		Pivots.Add(0, RootPivot);
		PivotOrder.Add(0);
	}

	// Mark all the pivots that have children as such. We'll use this for naming.
	for (TPair<int32, FPivot>& Item: Pivots)
	{
		const int32 ParentIndex = Item.Value.Parent;
		if (Pivots.Contains(ParentIndex))
		{
			Pivots[ParentIndex].bHasChildren = true;
		}
	}

	PivotOrder.Sort([Pivots](const int32 IndexA, const int32 IndexB)
	{
		return Pivots[IndexA].Parent < Pivots[IndexB].Parent;
	});

	// Give the pivots bone names and adjust the positions so that they're local to their parent.
	int32 LeafId = 0, BranchId = 0, TrunkId = 0;
	for (const int32 PivotIndex: PivotOrder)
	{
		FPivot& Pivot = Pivots[PivotIndex];

		if (Pivot.Parent == INDEX_NONE)
		{
			Pivot.BoneName = FName("Root");
		}
		else if (Pivot.Parent == 0)
		{
			Pivot.BoneName = FName("Trunk", TrunkId++);
		}
		else if (Pivot.bHasChildren)
		{
			Pivot.BoneName = FName("Branch", BranchId++);
		}
		else
		{
			Pivot.BoneName = FName("Leaf", LeafId++);
		}
		
		int32 ParentIndex = Pivot.Parent;
		while (ParentIndex != INDEX_NONE)
		{
			Pivot.Pos -= Pivots[ParentIndex].Pos;
			ParentIndex = Pivots[ParentIndex].Parent;
		}
	}
	
	// Construct the skeleton.
	{
		// This is a scoped operation. 
		FReferenceSkeletonModifier SkeletonModifier(TargetSkeleton);

		for (const int32 PivotIndex: PivotOrder)
		{
			FPivot& Pivot = Pivots[PivotIndex];

			FMeshBoneInfo BoneInfo;
			BoneInfo.Name = Pivot.BoneName;
			BoneInfo.ExportName = Pivot.BoneName.ToString();
			if (Pivot.Parent != INDEX_NONE)
			{
				BoneInfo.ParentIndex = Pivots[Pivot.Parent].BoneIndex;
			}

			FTransform BonePose(FVector3d(Pivot.Pos));
		
			SkeletonModifier.Add(BoneInfo, BonePose);
			Pivot.BoneIndex = SkeletonModifier.FindBoneIndex(Pivot.BoneName);
		}
	}

	// Bind the vertices as needed. If multiple instances of the same vertex refer to the different UVs,
	// use that to smooth bind.
	FSkinWeightsVertexAttributesRef SkinWeights = MeshAttributes.GetVertexSkinWeights();
	TMap<FBoneIndexType, int32> BoneIndexCounts; 
	for (FVertexID VertexID: MeshDescription.Vertices().GetElementIDs())
	{
		BoneIndexCounts.Reset();
		for (FVertexInstanceID VertexInstanceID: MeshDescription.GetVertexVertexInstanceIDs(VertexID))
		{
			FVector2f UV = UVCoords.Get(VertexInstanceID, TreePivotUVIndex);
			const int32 PivotIndex = UVToPivot[UV];
			const FBoneIndexType BoneIndex = Pivots[PivotIndex].BoneIndex;

			if (BoneIndexCounts.Contains(BoneIndex))
			{
				BoneIndexCounts[BoneIndex]++;
			}
			else
			{
				BoneIndexCounts.Add(BoneIndex, 1);
			}
		}

		FBoneIndexType BoneIndexes[3];
		float BoneWeights[3];

		int32 Index = 0;
		for (const TTuple<FBoneIndexType, int32>& Item: BoneIndexCounts)
		{
			BoneIndexes[Index] = Item.Key;
			BoneWeights[Index] = Item.Value / 3.0f;
			Index++;
		}

		SkinWeights.Set(VertexID, UE::AnimationCore::FBoneWeights::Create(BoneIndexes, BoneWeights, BoneIndexCounts.Num()));  
	}
	
	FElementIDRemappings Remappings;
	MeshDescription.Compact(Remappings);
	
	// Copy the static mesh over and bind the skeletal mesh to the new ref skeleton.
	{
		FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshChange(TargetSkeletalMesh);

		TargetSkeletalMesh->PreEditChange( nullptr );
		TargetSkeletalMesh->SetRefSkeleton(TargetSkeleton->GetReferenceSkeleton());
		TargetSkeletalMesh->CalculateInvRefMatrices();

		TargetSkeletalMesh->SetNaniteSettings(TreeStaticMesh->GetNaniteSettings());

		FSkeletalMeshModel* ImportedModels = TargetSkeletalMesh->GetImportedModel();
		ImportedModels->LODModels.Add(new FSkeletalMeshLODModel);
		
		FSkeletalMeshLODInfo& LODInfo = TargetSkeletalMesh->AddLODInfo();
		
		// Make sure there's no reduction active.
		LODInfo.ReductionSettings.NumOfTrianglesPercentage = 1.0f;
		LODInfo.ReductionSettings.MaxNumOfTriangles = MAX_uint32;
		LODInfo.ReductionSettings.MaxNumOfTrianglesPercentage = MAX_uint32;
		LODInfo.ReductionSettings.NumOfVertPercentage = 1.0f;
		LODInfo.ReductionSettings.MaxNumOfVerts = MAX_uint32;
		LODInfo.ReductionSettings.MaxNumOfVertsPercentage = MAX_uint32;
		LODInfo.ReductionSettings.MaxDeviationPercentage = 0.0f;

		TargetSkeletalMesh->CreateMeshDescription(0, MoveTemp(MeshDescription));
		TargetSkeletalMesh->CommitMeshDescription(0);

		// Copy the materials over.
		TArray<FSkeletalMaterial> Materials;
		for (const FStaticMaterial& StaticMaterial: TreeStaticMesh->GetStaticMaterials())
		{
			FSkeletalMaterial Material(
				StaticMaterial.MaterialInterface,
				StaticMaterial.MaterialSlotName);
		
			Materials.Add(Material);
		}

		TargetSkeletalMesh->SetMaterials(Materials);

		TargetSkeletalMesh->SetPositiveBoundsExtension(TreeStaticMesh->GetPositiveBoundsExtension());
		TargetSkeletalMesh->SetNegativeBoundsExtension(TreeStaticMesh->GetNegativeBoundsExtension());
	}

	TargetSkeletalMesh->SetSkeleton(TargetSkeleton);
	TargetSkeleton->MergeAllBonesToBoneTree(TargetSkeletalMesh);
	TargetSkeleton->SetPreviewMesh(TargetSkeletalMesh);
	return true;
}

bool UDynamicWindBlueprintLibrary::ImportDynamicWindSkeletalDataFromFile(USkeletalMesh* TargetSkeletalMesh)
{
	if (TargetSkeletalMesh == nullptr)
	{
		return false;
	}

	FString DefaultDirectory;
	if (auto AssetImportData = TargetSkeletalMesh->GetAssetImportData())
	{
		DefaultDirectory = FPaths::GetPath(AssetImportData->GetFirstFilename());
	}
	else
	{
		DefaultDirectory = FPaths::ProjectContentDir();
	}

	TArray<FString> FileNames;
	if (FDesktopPlatformModule::Get()->OpenFileDialog(
		FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
		TEXT("Import Dynamic Wind JSON"),
		DefaultDirectory,
		TEXT(""),
		TEXT("JSON File (*.json)|*.json"),
		EFileDialogFlags::None,
		FileNames))
	{
		FString JsonString;
		if (FFileHelper::LoadFileToString(JsonString, *FileNames[0]))
		{
			FDynamicWindSkeletalImportData ImportData;
			if (FJsonObjectConverter::JsonObjectStringToUStruct(JsonString, &ImportData))
			{
				if (DynamicWind::ImportSkeletalData(*TargetSkeletalMesh, ImportData))
				{
					return true;
				}
			}
		}

		UE_LOGF(LogDynamicWindBlueprintLibrary, Error,
			"ImportDynamicWindSkeletalDataFromFile: Failed to load dynamic wind skeletal data from %ls for SkeletalMesh %ls.",
			*FileNames[0], *TargetSkeletalMesh->GetFullName()
			);
	}

	return false;
}
