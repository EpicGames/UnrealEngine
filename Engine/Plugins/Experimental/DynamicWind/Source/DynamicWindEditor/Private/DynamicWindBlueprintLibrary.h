// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicWindImportData.h"

#include "DynamicWindBlueprintLibrary.generated.h"

class USkeletalMesh;
class UStaticMesh;
class UTexture2D;

UCLASS()
class UDynamicWindBlueprintLibrary : public UObject
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category="DynamicWind")
	static bool ConvertPivotPainterTreeToSkeletalMesh(
		const UStaticMesh* TreeStaticMesh,
		UTexture2D* TreePivotPosTexture,
		int32 TreePivotUVIndex,
		USkeletalMesh* TargetSkeletalMesh,
		USkeleton* TargetSkeleton
	);

	UFUNCTION(BlueprintCallable, Category="DynamicWind")
	static bool ImportDynamicWindSkeletalDataFromFile(USkeletalMesh* TargetSkeletalMesh);
};
