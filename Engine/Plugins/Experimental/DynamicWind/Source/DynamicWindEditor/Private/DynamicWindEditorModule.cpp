// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicWindEditorModule.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "DynamicWindEditor"

void FDynamicWindEditorModule::StartupModule()
{
	//FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
	//IAssetTools& AssetTools = AssetToolsModule.Get();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

}
	
void FDynamicWindEditorModule::ShutdownModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

}

FDynamicWindEditorModule& FDynamicWindEditorModule::GetModule()
{
	static const FName ModuleName = "DynamicWindEditor";
	return FModuleManager::LoadModuleChecked<FDynamicWindEditorModule>(ModuleName);
}

IMPLEMENT_MODULE(FDynamicWindEditorModule, DynamicWindEditor);

#undef LOCTEXT_NAMESPACE
