// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "UObject/StrongObjectPtr.h"

class UPackage;
class FString;

class FDynamicWindEditorModule : public IModuleInterface
{
public:

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	static FDynamicWindEditorModule& GetModule();

private:

};
