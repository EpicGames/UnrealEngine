// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicWindFactory.h"
#include "DynamicWindData.h"

UDynamicWindDataFactory::UDynamicWindDataFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	SupportedClass = UDynamicWindData::StaticClass();
	ProviderDataClass = UDynamicWindData::StaticClass();
}

bool UDynamicWindDataFactory::ConfigureProperties()
{
	return true;
}
