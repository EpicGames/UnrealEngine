// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicWindImportData.h"
#include "Engine/SkeletalMesh.h"

namespace DynamicWind
{

UDynamicWindSkeletalData* ImportSkeletalData(
	USkeletalMesh& TargetSkeletalMesh,
	const FDynamicWindSkeletalImportData& ImportData
)
{
	const FReferenceSkeleton& SkeletonRef = TargetSkeletalMesh.GetRefSkeleton();
	const TArray<FMeshBoneInfo>& MeshBoneInfos = SkeletonRef.GetRawRefBoneInfo();

	struct FBoneInfo
	{
		int32 ID = INDEX_NONE;
		int32 ParentID = INDEX_NONE;
		FName Name = NAME_None;
	
		int32 SimulationGroupIndex = -1;

		FVector3f Start = FVector3f::Zero();
		FVector3f End = FVector3f::Zero();
	};

	int32 MaxSimulationGroupIndex = -1;
	TArray<FBoneInfo> BoneInfos;
	BoneInfos.Reserve(MeshBoneInfos.Num());
	for (auto Bone : MeshBoneInfos)
	{
		int Index = SkeletonRef.FindRawBoneIndex(Bone.Name);

		if (Index != INDEX_NONE)
		{
			FBoneInfo NewBone;
			NewBone.ID = Index;
			NewBone.ParentID = Bone.ParentIndex;
			NewBone.Name = Bone.Name;

			int JointIndex = ImportData.Joints.IndexOfByPredicate(
				[BoneName=Bone.Name] (const FDynamicWindJointImportData& Joint)
				{
					return Joint.JointName == BoneName;
				}
			);

			if (JointIndex != INDEX_NONE)
			{
				const FDynamicWindJointImportData& Joint = ImportData.Joints[JointIndex];
				NewBone.SimulationGroupIndex = Joint.SimulationGroupIndex;

				MaxSimulationGroupIndex = FMath::Max(MaxSimulationGroupIndex, NewBone.SimulationGroupIndex);
			}

			const auto& RawRefBonePose = SkeletonRef.GetRawRefBonePose();
			NewBone.Start = RawRefBonePose.IsValidIndex(Bone.ParentIndex) ?
				FVector3f(RawRefBonePose[Bone.ParentIndex].GetLocation()) : FVector3f::Zero();
			NewBone.End = FVector3f(RawRefBonePose[Index].GetLocation());

			BoneInfos.Add(NewBone);
		}
	}

	auto AssetUserData = TargetSkeletalMesh.GetAssetUserData<UDynamicWindSkeletalData>();
	if (AssetUserData == nullptr)
	{
		AssetUserData = NewObject<UDynamicWindSkeletalData>(&TargetSkeletalMesh);
		TargetSkeletalMesh.AddAssetUserData(AssetUserData);
	}
	else
	{
		// Reset the user data
		AssetUserData->SimulationGroups.Reset();
		AssetUserData->SimulationGroupBones.Reset();
		AssetUserData->BoneChains.Reset();
		AssetUserData->ExtraBonesData.Reset();
	}

	AssetUserData->bIsEnabled			= true;
	AssetUserData->bIsGroundCover		= ImportData.bIsGroundCover;
	AssetUserData->GustAttenuation		= ImportData.GustAttenuation;
	AssetUserData->SimulationGroups		= ImportData.SimulationGroups;

	// To be fault tolerant, add default data for simulation groups assigned to joints that are not represented
	// in the import data
	TMap<int32, TSet<int32>> SimulationGroupBones;
	while (AssetUserData->SimulationGroups.Num() < MaxSimulationGroupIndex + 1)
	{
		AssetUserData->SimulationGroups.Add({
			.Influence = 1.0f
		});
	}

	for (const auto& BoneInfo : BoneInfos)
	{
		if (BoneInfo.SimulationGroupIndex < 0)
		{
			continue;
		}

		SimulationGroupBones.FindOrAdd(BoneInfo.SimulationGroupIndex).Add(BoneInfo.ID);

		// Find bone chain origin bone and this bone's position in chain
		FDynamicWindExtraBoneData ExtraBoneData
		{
			.BoneChainOriginBoneIndex = BoneInfo.ID
		};
		int32 CurrentParent = BoneInfo.ParentID;
		while (CurrentParent != INDEX_NONE)
		{
			if (BoneInfos[CurrentParent].SimulationGroupIndex != BoneInfo.SimulationGroupIndex)
			{
				break;
			}

			ExtraBoneData.BoneChainOriginBoneIndex = BoneInfos[CurrentParent].ID;
			ExtraBoneData.IndexInBoneChain++;
			
			CurrentParent = BoneInfos[CurrentParent].ParentID;
		}
		
		auto& ChainInfo = AssetUserData->BoneChains.FindOrAdd(ExtraBoneData.BoneChainOriginBoneIndex);
		ChainInfo.NumBones++;
		ChainInfo.ChainLength += FVector3f::Distance(BoneInfo.Start, BoneInfo.End);
		AssetUserData->ExtraBonesData.Emplace(BoneInfo.ID, ExtraBoneData);
	}

	for (auto& [SimulationGroupIndex, BoneIndices] : SimulationGroupBones)
	{
		AssetUserData->SimulationGroupBones.Add({ SimulationGroupIndex, MoveTemp(BoneIndices) });
	}
	
	AssetUserData->RecalculateSkeletalDataHash();
	
	TargetSkeletalMesh.MarkPackageDirty();
	TargetSkeletalMesh.PostEditChange();
	TargetSkeletalMesh.Modify();

	return AssetUserData;
}

} // namespace DynamicWind