// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "Factories/TransformProviderFactory.h"
#include "DynamicWindFactory.generated.h"

UCLASS(MinimalAPI)
class UDynamicWindDataFactory : public UTransformProviderDataFactory
{
	GENERATED_UCLASS_BODY()

	virtual bool ConfigureProperties() override;
	virtual bool ShouldShowInNewMenu() const override { return false; }
};
