// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicWindSkeletalData.h"

#include "DynamicWindImportData.generated.h"

class USkeletalMesh;

USTRUCT()
struct DYNAMICWINDEDITOR_API FDynamicWindJointImportData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Default")
	FName JointName;

	UPROPERTY(EditAnywhere, Category = "Default")
	int32 SimulationGroupIndex = 0;
};

USTRUCT(BlueprintType)
struct DYNAMICWINDEDITOR_API FDynamicWindSkeletalImportData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "SkeletalData")
	TArray<FDynamicWindJointImportData> Joints;

	UPROPERTY(EditAnywhere, Category = "SkeletalData")
	TArray<FDynamicWindSimulationGroupData> SimulationGroups;

	UPROPERTY(EditAnywhere, Category = "SkeletalData")
	bool bIsGroundCover = false;

	UPROPERTY(EditAnywhere, Category = "SkeletalData")
	float GustAttenuation = 0.0f;
};

namespace DynamicWind
{

/** Attempts to create dynamic wind asset user data for a skeletal mesh with the given import data */
DYNAMICWINDEDITOR_API UDynamicWindSkeletalData* ImportSkeletalData(
	USkeletalMesh& TargetSkeletalMesh,
	const FDynamicWindSkeletalImportData& ImportData
);

}
